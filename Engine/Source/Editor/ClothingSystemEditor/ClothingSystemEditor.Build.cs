// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class ClothingSystemEditor : ModuleRules
{
	public ClothingSystemEditor(ReadOnlyTargetRules Target) : base(Target)
	{
        PublicIncludePathModuleNames.Add("UnrealEd");
        PublicIncludePathModuleNames.Add("ClothingSystemRuntimeInterface");
        PublicIncludePathModuleNames.Add("ClothingSystemRuntimeNv");
        PublicIncludePathModuleNames.Add("ClothingSystemEditorInterface");
		PublicIncludePathModuleNames.Add("Persona");

		PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "Engine",
                "RenderCore"
            }
        );

        PublicDependencyModuleNames.AddRange(
			new string[] { 
				"Core",
				"CoreUObject",
				"ClothingSystemRuntimeInterface",
                "ClothingSystemRuntimeCommon",
                "ClothingSystemRuntimeNv",
                "ContentBrowser",
				"EditorFramework",
                "UnrealEd",
                "SlateCore",
                "Slate",
                "ClothingSystemEditorInterface"
            }
		);

        SetupModulePhysicsSupport(Target);
	}
}
