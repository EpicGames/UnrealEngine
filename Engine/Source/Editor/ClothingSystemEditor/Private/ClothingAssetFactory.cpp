// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingAssetFactory.h"

#include "BoneIndices.h"
#include "BoneWeights.h"
#include "ClothLODData.h"
#include "ClothPhysicalMeshData.h"
#include "ClothVertBoneData.h"
#include "ClothingAsset.h"
#include "ClothingAssetBase.h"
#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "Containers/IndirectArray.h"
#include "CoreGlobals.h"
#include "Engine/SkeletalMesh.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GPUSkinPublicDefs.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Logging/LogCategory.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Guid.h"
#include "ObjectTools.h"
#include "PhysicsEngine/PhysicsAsset.h"
#include "PointWeightMap.h"
#include "ReferenceSkeleton.h"
#include "Rendering/SkeletalMeshLODModel.h"
#include "Rendering/SkeletalMeshModel.h"
#include "Serialization/StructuredArchiveAdapters.h"
#include "SkeletalMeshTypes.h"
#include "Templates/Casts.h"
#include "Templates/UnrealTemplate.h"
#include "Trace/Detail/Channel.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Utils/ClothingMeshUtils.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClothingAssetFactory)

#define LOCTEXT_NAMESPACE "ClothingAssetFactory"
DEFINE_LOG_CATEGORY(LogClothingAssetFactory)

using namespace nvidia::apex;

namespace ClothingFactoryConstants
{
	// For verifying the file 
	static const char ClothingAssetClass[] = "ClothingAssetParameters";

	// Import transformation params
	static const char ParamName_BoneActors[] = "boneActors";
	static const char ParamName_BoneSpheres[] = "boneSpheres";
	static const char ParamName_GravityDirection[] = "simulation.gravityDirection";
	static const char ParamName_UvOrigin[] = "textureUVOrigin";

	// UV flip params
	static const char ParamName_SubmeshArray[] = "submeshes";
	static const char ParamName_SubmeshBufferFormats[] = "vertexBuffer.vertexFormat.bufferFormats";
	static const char ParamName_VertexBuffers[] = "vertexBuffer.buffers";
	static const char ParamName_Semantic[] = "semantic";
	static const char ParamName_BufferData[] = "data";

	static const char ParamName_GLOD_Platforms[] = "platforms";
	static const char ParamName_GLOD_LOD[] = "lod";
	static const char ParamName_GLOD_PhysMeshID[] = "physicalMeshId";
	static const char ParamName_GLOD_RenderMeshAsset[] = "renderMeshAsset";
	static const char ParamName_GLOD_ImmediateClothMap[] = "immediateClothMap";
	static const char ParamName_GLOD_SkinClothMapB[] = "SkinClothMapB";
	static const char ParamName_GLOD_SkinClothMap[] = "SkinClothMap";
	static const char ParamName_GLOD_SkinClothMapThickness[] = "skinClothMapThickness";
	static const char ParamName_GLOD_SkinClothMapOffset[] = "skinClothMapOffset";
	static const char ParamName_GLOD_TetraMap[] = "tetraMap";
	static const char ParamName_GLOD_RenderMeshAssetSorting[] = "renderMeshAssetSorting";
	static const char ParamName_GLOD_PhysicsMeshPartitioning[] = "physicsMeshPartitioning";

	static const char ParamName_Partition_GraphicalSubmesh[] = "graphicalSubmesh";
	static const char ParamName_Partition_NumSimVerts[] = "numSimulatedVertices";
	static const char ParamName_Partition_NumSimVertsAdditional[] = "numSimulatedVerticesAdditional";
	static const char ParamName_Partition_NumSimIndices[] = "numSimulatedIndices";
}

void LogAndToastWarning(const FText& Error)
{
	FNotificationInfo Info(Error);
	Info.ExpireDuration = 5.0f;
	FSlateNotificationManager::Get().AddNotification(Info);

	UE_LOGF(LogClothingAssetFactory, Warning, "%ls", *Error.ToString());
}

UClothingAssetFactory::UClothingAssetFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

const UClass* UClothingAssetFactory::GetSupportedSourceAssetType() const
{
	return USkeletalMesh::StaticClass();
}

TArray<UClothingAssetBase*> UClothingAssetFactory::CreateFromSourceAsset(USkeletalMesh* TargetMesh, const UObject* SourceAsset) const
{
	TArray<UClothingAssetBase*> ClothingAssets;
	if (const USkeletalMesh* const SourceSkeletalMesh = Cast<USkeletalMesh>(SourceAsset))
	{
		for (const UClothingAssetBase* const ClothingAsset : SourceSkeletalMesh->GetMeshClothingAssets())
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS  // UE_DEPRECATED(5.7, ""): CreateFromExistingCloth will be moved out of UClothingAssetFactoryBase to this class
			if (UClothingAssetBase* const NewAsset =
				const_cast<UClothingAssetFactory*>(this)->CreateFromExistingCloth(
					TargetMesh,
					const_cast<USkeletalMesh*>(SourceSkeletalMesh),
					const_cast<UClothingAssetBase*>(ClothingAsset)))
{
				ClothingAssets.Emplace(NewAsset);
			}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
	return ClothingAssets;
}

UClothingAssetBase* UClothingAssetFactory::CreateFromSkeletalMesh(USkeletalMesh* TargetMesh, FSkeletalMeshClothBuildParams& Params)
{
	// Need a valid skel mesh
	if(!TargetMesh)
	{
		return nullptr;
	}

	FSkeletalMeshModel* Mesh = TargetMesh->GetImportedModel();

	// Need a valid resource
	if(!Mesh)
	{
		return nullptr;
	}

	// Need a valid LOD model
	if(!Mesh->LODModels.IsValidIndex(Params.LodIndex))
	{
		return nullptr;
	}

	FSkeletalMeshLODModel& LodModel = Mesh->LODModels[Params.LodIndex];

	// Need a valid section
	if(!LodModel.Sections.IsValidIndex(Params.SourceSection))
	{
		return nullptr;
	}

	// Ok, we have a valid mesh and section, we can now extract it as a sim mesh
	FSkelMeshSection& SourceSection = LodModel.Sections[Params.SourceSection];

	// Can't convert to a clothing asset if bound to clothing
	if(SourceSection.HasClothingData())
	{
		return nullptr;
	}

	const FString SanitizedName = ObjectTools::SanitizeObjectName(Params.AssetName);
	const FName ObjectName = StaticFindObject(UClothingAssetBase::StaticClass(), TargetMesh, *SanitizedName) ?
		MakeUniqueObjectName(TargetMesh, UClothingAssetBase::StaticClass(), FName(SanitizedName)) :
		FName(SanitizedName);

	UClothingAssetCommon* NewAsset = NewObject<UClothingAssetCommon>(TargetMesh, ObjectName);
	NewAsset->SetFlags(RF_Transactional);

	// Adding a new LOD from this skeletal mesh
	NewAsset->AddNewLod();
	FClothLODDataCommon& LodData = NewAsset->LodData.Last();

	if(ImportToLodInternal(TargetMesh, Params.LodIndex, Params.SourceSection, NewAsset, LodData, Params.LodIndex))  // Use the same LOD index as both source and destination index
	{
		FScopedSkeletalMeshPostEditChange ScopedSkeletalMeshPostEditChange(TargetMesh);
		if(Params.bRemoveFromMesh)
		{
			// User doesn't want the section anymore as a renderable, get rid of it
			TargetMesh->RemoveMeshSection(Params.LodIndex, Params.SourceSection);  // Note: this is now taken care of ahead of the call to this function in order to get the correct used bone array, left here to not break the API behavior
		}

		// Set asset guid
		NewAsset->AssetGuid = FGuid::NewGuid();

		// Set physics asset, will be used when building actors for cloth collisions
		NewAsset->PhysicsAsset = Params.PhysicsAsset.LoadSynchronous();

		// Build the final bone map
		NewAsset->RefreshBoneMapping(TargetMesh);

		// Invalidate cached data as the mesh has changed
		NewAsset->InvalidateAllCachedData();

		return NewAsset;
	}

	return nullptr;
}

UClothingAssetBase* UClothingAssetFactory::CreateFromExistingCloth(USkeletalMesh* TargetMesh, const USkeletalMesh* /*SourceMesh*/, const UClothingAssetBase* SourceAsset) const
{
	const UClothingAssetCommon* const SourceClothingAsset = Cast<const UClothingAssetCommon>(SourceAsset);

	if (!SourceClothingAsset)
	{
		return nullptr;
	}

	//Duplicating the clothing asset using the existing asset as a template
	const FName NewAssetName = StaticFindObject(UClothingAssetBase::StaticClass(), TargetMesh, *SourceClothingAsset->GetName()) ?
		MakeUniqueObjectName(TargetMesh, UClothingAssetBase::StaticClass(), SourceClothingAsset->GetFName()) :
		SourceClothingAsset->GetFName();

	UClothingAssetCommon* const NewAsset = DuplicateObject<UClothingAssetCommon>(SourceClothingAsset, TargetMesh, NewAssetName);

	NewAsset->AssetGuid = FGuid::NewGuid();
	//Need to empty LODMap to remove previous mappings from cloth LOD to SkelMesh LOD
	NewAsset->LodMap.Empty();
	NewAsset->RefreshBoneMapping(TargetMesh);
	NewAsset->InvalidateAllCachedData();

	return NewAsset;
}

UClothingAssetBase* UClothingAssetFactory::ImportLodToClothing(USkeletalMesh* TargetMesh, FSkeletalMeshClothBuildParams& Params)
{
	if(!TargetMesh)
	{
		// Invalid target - can't continue.
		LogAndToastWarning(LOCTEXT("Warning_InvalidLodMesh", "Failed to import clothing LOD, invalid target mesh specified"));
		return nullptr;
	}

	if(!Params.TargetAsset.IsValid())
	{
		// Invalid target - can't continue.
		LogAndToastWarning(LOCTEXT("Warning_InvalidClothTarget", "Failed to import clothing LOD, invalid target clothing object"));
		return nullptr;
	}

	FSkeletalMeshModel* MeshResource = TargetMesh->GetImportedModel();
	check(MeshResource);

	const int32 NumMeshLods = MeshResource->LODModels.Num();

	if(UClothingAssetBase* TargetClothing = Params.TargetAsset.Get())
	{
		// Find the clothing asset in the mesh to verify the params are correct
		int32 MeshAssetIndex = INDEX_NONE;
		if(TargetMesh->GetMeshClothingAssets().Find(TargetClothing, MeshAssetIndex))
		{
			// Everything looks good, continue to actual import
			if(UClothingAssetCommon* const ConcreteTarget = Cast<UClothingAssetCommon>(TargetClothing))
			{
			const FClothLODDataCommon* RemapSource = nullptr;

			if(Params.bRemapParameters)
			{
				if(Params.TargetLod == ConcreteTarget->GetNumLods())
				{
					// New LOD, remap from previous
					RemapSource = &ConcreteTarget->LodData.Last();
				}
				else
				{
					// This is a replacement, remap from current LOD
					check(ConcreteTarget->LodData.IsValidIndex(Params.TargetLod));
					RemapSource = &ConcreteTarget->LodData[Params.TargetLod];
				}
			}

			if(Params.TargetLod == ConcreteTarget->GetNumLods())
			{
				ConcreteTarget->AddNewLod();
			}
			else if(!ConcreteTarget->LodData.IsValidIndex(Params.TargetLod))
			{
				LogAndToastWarning(LOCTEXT("Warning_InvalidLodTarget", "Failed to import clothing LOD, invalid target LOD."));
				return nullptr;
			}

			FClothLODDataCommon& NewLod = ConcreteTarget->LodData[Params.TargetLod];

			if(Params.TargetLod > 0 && Params.bRemapParameters)
			{
				RemapSource = &ConcreteTarget->LodData[Params.TargetLod - 1];
			}

			if(ImportToLodInternal(TargetMesh, Params.LodIndex, Params.SourceSection, ConcreteTarget, NewLod, Params.TargetLod, RemapSource))
			{
				// Rebuild the final bone map
				ConcreteTarget->RefreshBoneMapping(TargetMesh);

				// Build Lod skinning map for smooth transitions
				ConcreteTarget->BuildLodTransitionData();

				// Invalidate cached data as the mesh has changed
				ConcreteTarget->InvalidateAllCachedData();

				return ConcreteTarget;
			}
		}
	}
}

	return nullptr;
}

bool UClothingAssetFactory::ImportToLodInternal(
	USkeletalMesh* SourceMesh, 
	int32 SourceLodIndex, 
	int32 SourceSectionIndex, 
	UClothingAssetCommon* DestAsset, 
	FClothLODDataCommon& DestLod, 
	int32 DestLodIndex, 
	const FClothLODDataCommon* InParameterRemapSource)
{
	if(!SourceMesh || !SourceMesh->GetImportedModel())
	{
		// Invalid mesh
		return false;
	}

	FSkeletalMeshModel* SkeletalResource = SourceMesh->GetImportedModel();

	if(!SkeletalResource->LODModels.IsValidIndex(SourceLodIndex))
	{
		// Invalid LOD
		return false;
	}

	FSkeletalMeshLODModel& SourceLod = SkeletalResource->LODModels[SourceLodIndex];

	if(!SourceLod.Sections.IsValidIndex(SourceSectionIndex))
	{
		// Invalid Section
		return false;
	}

	FSkelMeshSection& SourceSection = SourceLod.Sections[SourceSectionIndex];

	const int32 NumVerts = SourceSection.SoftVertices.Num();
	const int32 NumIndices = SourceSection.NumTriangles * 3;
	const int32 BaseIndex = SourceSection.BaseIndex;
	const int32 BaseVertexIndex = SourceSection.BaseVertexIndex;

	// We need to weld the mesh verts to get rid of duplicates (happens for smoothing groups)
	TArray<FVector> UniqueVerts;
	TArray<uint32> OriginalIndexes;

	TArray<uint32> IndexRemap;
	IndexRemap.AddDefaulted(NumVerts);
	{
		static const float ThreshSq = SMALL_NUMBER * SMALL_NUMBER;
		for(int32 VertIndex = 0; VertIndex < NumVerts; ++VertIndex)
		{
			const FSoftSkinVertex& SourceVert = SourceSection.SoftVertices[VertIndex];

			bool bUnique = true;
			int32 RemapIndex = INDEX_NONE;

			const int32 NumUniqueVerts = UniqueVerts.Num();
			for(int32 UniqueVertIndex = 0; UniqueVertIndex < NumUniqueVerts; ++UniqueVertIndex)
			{
				FVector& UniqueVert = UniqueVerts[UniqueVertIndex];

				if((UniqueVert - (FVector)SourceVert.Position).SizeSquared() <= ThreshSq)
				{
					// Not unique
					bUnique = false;
					RemapIndex = UniqueVertIndex;

					break;
				}
			}

			if(bUnique)
			{
				// Unique
				UniqueVerts.Add((FVector)SourceVert.Position);
				OriginalIndexes.Add(VertIndex);
				IndexRemap[VertIndex] = UniqueVerts.Num() - 1;
			}
			else
			{
				IndexRemap[VertIndex] = RemapIndex;
			}
		}
	}

	const int32 NumUniqueVerts = UniqueVerts.Num();

	// If we're going to remap the parameters we need to cache the remap source
	// data. We copy it here incase the destination and remap source
	// lod models are aliased (as in a reimport)
	TArray<FVector3f> CachedPositions;
	TArray<FVector3f> CachedNormals;
	TArray<uint32> CachedIndices;
	int32 NumSourceMasks = 0;
	TArray<FPointWeightMap> SourceMaskCopy;
	
	bool bPerformParamterRemap = false;

	if(InParameterRemapSource)
	{
		const FClothPhysicalMeshData& RemapPhysMesh = InParameterRemapSource->PhysicalMeshData;
		CachedPositions = RemapPhysMesh.Vertices;
		CachedNormals = RemapPhysMesh.Normals;
		CachedIndices = RemapPhysMesh.Indices;
		SourceMaskCopy = InParameterRemapSource->PointWeightMaps;
		NumSourceMasks = SourceMaskCopy.Num();

		bPerformParamterRemap = true;
	}

	FClothPhysicalMeshData& PhysMesh = DestLod.PhysicalMeshData;
	PhysMesh.Reset(NumUniqueVerts, NumIndices);

	const FSkeletalMeshLODModel* const DestLodModel =
		SkeletalResource->LODModels.IsValidIndex(DestLodIndex) ?  // The dest section LOD level might not exist yet, that shouldn't prevent a cloth asset LOD creation
		&SkeletalResource->LODModels[DestLodIndex] : nullptr;

	for(int32 VertexIndex = 0; VertexIndex < NumUniqueVerts; ++VertexIndex)
	{
		const FSoftSkinVertex& SourceVert = SourceSection.SoftVertices[OriginalIndexes[VertexIndex]];

		PhysMesh.Vertices[VertexIndex] = SourceVert.Position;
		PhysMesh.Normals[VertexIndex] = SourceVert.TangentZ;
		PhysMesh.VertexColors[VertexIndex] = SourceVert.Color;

		FClothVertBoneData& BoneData = PhysMesh.BoneData[VertexIndex];
		for(int32 InfluenceIndex = 0; InfluenceIndex < MAX_TOTAL_INFLUENCES; ++InfluenceIndex)
		{
			uint16 SourceIndex = SourceSection.BoneMap[SourceVert.InfluenceBones[InfluenceIndex]];

			// If the current bone is not active in the destination LOD, then remap to the first ancestor bone that is
			while (DestLodModel && !DestLodModel->ActiveBoneIndices.Contains(SourceIndex))
			{
				SourceIndex = SourceMesh->GetRefSkeleton().GetParentIndex(SourceIndex);
			}

			if(SourceIndex != INDEX_NONE)
			{
				FName BoneName = SourceMesh->GetRefSkeleton().GetBoneName(SourceIndex);
				BoneData.BoneIndices[InfluenceIndex] = DestAsset->UsedBoneNames.AddUnique(BoneName);
				BoneData.BoneWeights[InfluenceIndex] = (float)SourceVert.InfluenceWeights[InfluenceIndex] / UE::AnimationCore::MaxRawBoneWeightFloat;
			}
		}
	}

	// Add a max distance parameter mask to the physics mesh
	FPointWeightMap& PhysMeshMaxDistances = PhysMesh.AddWeightMap(EWeightMapTargetCommon::MaxDistance);
	PhysMeshMaxDistances.Initialize(PhysMesh.Vertices.Num());

	// Add a max distance parameter mask to the LOD
	DestLod.PointWeightMaps.AddDefaulted();
	FPointWeightMap& LodMaxDistances = DestLod.PointWeightMaps.Last();
	LodMaxDistances.Initialize(PhysMeshMaxDistances, EWeightMapTargetCommon::MaxDistance);

	PhysMesh.MaxBoneWeights = SourceSection.MaxBoneInfluences;

	for(int32 IndexIndex = 0; IndexIndex < NumIndices; ++IndexIndex)
	{
		PhysMesh.Indices[IndexIndex] = SourceLod.IndexBuffer[BaseIndex + IndexIndex] - BaseVertexIndex;
		PhysMesh.Indices[IndexIndex] = IndexRemap[PhysMesh.Indices[IndexIndex]];
	}

	// Validate the generated triangles. If the source mesh has colinear triangles then clothing simulation will fail
	const int32 NumTriangles = PhysMesh.Indices.Num() / 3;
	for(int32 TriIndex = 0; TriIndex < NumTriangles; ++TriIndex)
	{
		FVector A = (FVector)PhysMesh.Vertices[PhysMesh.Indices[TriIndex * 3 + 0]];
		FVector B = (FVector)PhysMesh.Vertices[PhysMesh.Indices[TriIndex * 3 + 1]];
		FVector C = (FVector)PhysMesh.Vertices[PhysMesh.Indices[TriIndex * 3 + 2]];

		FVector TriNormal = (B - A) ^ (C - A);
		if(TriNormal.SizeSquared() <= SMALL_NUMBER)
		{
			// This triangle is colinear
			LogAndToastWarning(FText::Format(LOCTEXT("Colinear_Error", "Failed to generate clothing sim mesh due to degenerate triangle, found conincident vertices in triangle A={0} B={1} C={2}"), FText::FromString(A.ToString()), FText::FromString(B.ToString()), FText::FromString(C.ToString())));

			return false;
		}
	}

	if(bPerformParamterRemap)
	{
		ClothingMeshUtils::FVertexParameterMapper ParameterRemapper(PhysMesh.Vertices, PhysMesh.Normals, CachedPositions, CachedNormals, CachedIndices);

		DestLod.PointWeightMaps.Reset(NumSourceMasks);

		for(int32 MaskIndex = 0; MaskIndex < NumSourceMasks; ++MaskIndex)
		{
			const FPointWeightMap& SourceMask = SourceMaskCopy[MaskIndex];

			DestLod.PointWeightMaps.AddDefaulted();
			FPointWeightMap& DestMask = DestLod.PointWeightMaps.Last();

			DestMask.Initialize(PhysMesh.Vertices.Num());
			DestMask.CurrentTarget = SourceMask.CurrentTarget;
			DestMask.bEnabled = SourceMask.bEnabled;

			ParameterRemapper.Map(SourceMask.Values, DestMask.Values);
		}

		DestAsset->ApplyParameterMasks();
	}

	int32 LODVertexBudget;
	if (GConfig->GetInt(TEXT("ClothSettings"), TEXT("LODVertexBudget"), LODVertexBudget, GEditorIni) && LODVertexBudget > 0 && NumUniqueVerts > LODVertexBudget)
	{
		LogAndToastWarning(FText::Format(LOCTEXT("LODVertexBudgetWarning", "This cloth LOD has {0} more vertices than what is budgeted on this project (current={1}, budget={2})"), 
			NumUniqueVerts - LODVertexBudget, 
			NumUniqueVerts,
			LODVertexBudget));
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
