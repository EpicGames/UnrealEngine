// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClothingSystemEditorModule.h"

#include "ClothingAssetFactory.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"
#include "UObject/Class.h"

IMPLEMENT_MODULE(FClothingSystemEditorModule, ClothingSystemEditor);

FClothingSystemEditorModule::FClothingSystemEditorModule()
{

}

void FClothingSystemEditorModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(IClothingAssetFactoryProvider::FeatureName, this);
}

void FClothingSystemEditorModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(IClothingAssetFactoryProvider::FeatureName, this);
}

UClothingAssetFactoryBase* FClothingSystemEditorModule::GetFactory()
{
	return UClothingAssetFactory::StaticClass()->GetDefaultObject<UClothingAssetFactoryBase>();
}
