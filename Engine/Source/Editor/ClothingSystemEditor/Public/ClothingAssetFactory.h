// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClothingAsset.h"
#include "ClothingAssetFactoryInterface.h"
#include "Containers/UnrealString.h"
#include "GPUSkinPublicDefs.h"
#include "HAL/Platform.h"
#include "Logging/LogMacros.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"

#include "ClothingAssetFactory.generated.h"

#define UE_API CLOTHINGSYSTEMEDITOR_API

class UClothingAssetBase;
class UObject;
class USkeletalMesh;
struct FClothLODDataCommon;
struct FSkeletalMeshClothBuildParams;


DECLARE_LOG_CATEGORY_EXTERN(LogClothingAssetFactory, Log, All);

class FSkeletalMeshLODModel;
class UClothingAssetCommon;

namespace nvidia
{
	namespace apex
	{
		class ClothingAsset;
	}
}

namespace NvParameterized
{
	class Interface;
}

UCLASS(MinimalAPI, hidecategories=Object)
class UClothingAssetFactory : public UClothingAssetFactoryBase
{
	GENERATED_BODY()

public:

	UE_API UClothingAssetFactory(const FObjectInitializer& ObjectInitializer);

	UE_API virtual const UClass* GetSupportedSourceAssetType() const override;

	UE_API virtual TArray<UClothingAssetBase*> CreateFromSourceAsset(USkeletalMesh* TargetMesh, const UObject* SourceAsset) const override;

	UE_API virtual UClothingAssetBase* CreateFromSkeletalMesh(USkeletalMesh* TargetMesh, FSkeletalMeshClothBuildParams& Params) override;

	UE_API virtual UClothingAssetBase* ImportLodToClothing(USkeletalMesh* TargetMesh, FSkeletalMeshClothBuildParams& Params) override;

	// Deprecated methods, cannot be removed because of them being virtual
	UE_DEPRECATED(5.7, "Use CreateFromSupportedAsset instead.")
	virtual UClothingAssetBase* CreateFromExistingCloth(USkeletalMesh* TargetMesh, USkeletalMesh* SourceMesh, UClothingAssetBase* SourceAsset) override
	{
		return static_cast<const UClothingAssetFactory*>(this)->CreateFromExistingCloth(TargetMesh, SourceMesh, SourceAsset);
	}

private:

	// Utility methods for skeletal mesh extraction //////////////////////////

	/**
	 * Given a target mesh and a source clothing asset bound to source mesh, clone the clothing asset for
	 * use on the target mesh
	 * @param TargetMesh - The mesh to target
	 * @param SourceMesh - The mesh to copy the clothing asset from
	 * @param SourceAsset - The clothing asset to copy
	 */
	UClothingAssetBase* CreateFromExistingCloth(USkeletalMesh* TargetMesh, const USkeletalMesh* SourceMesh, const UClothingAssetBase* SourceAsset) const;

	/** Handles internal import of LODs */
	bool ImportToLodInternal(USkeletalMesh* SourceMesh, int32 SourceLodIndex, int32 SourceSectionIndex, UClothingAssetCommon* DestAsset, FClothLODDataCommon& DestLod, int32 DestLodIndex, const FClothLODDataCommon* InParameterRemapSource = nullptr);

	//////////////////////////////////////////////////////////////////////////

};

#undef UE_API
