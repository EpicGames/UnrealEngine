// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClothingAssetFactoryInterface.h"
#include "Modules/ModuleInterface.h"
#include "SimulationEditorExtenderNv.h"

class UClothingAssetFactoryBase;

class FClothingSystemEditorModule : public IModuleInterface, public IClothingAssetFactoryProvider
{

public:

	FClothingSystemEditorModule();

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual UClothingAssetFactoryBase* GetFactory() override;

private:

};
